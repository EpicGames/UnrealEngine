// Copyright Epic Games, Inc. All Rights Reserved.

// This code is largely based on that in ir_print_glsl_visitor.cpp from
// glsl-optimizer.
// https://github.com/aras-p/glsl-optimizer
// The license for glsl-optimizer is reproduced below:

/*
	GLSL Optimizer is licensed according to the terms of the MIT license:

	Copyright (C) 1999-2007  Brian Paul   All Rights Reserved.
	Copyright (C) 2010-2011  Unity Technologies All Rights Reserved.

	Permission is hereby granted, free of charge, to any person obtaining a
	copy of this software and associated documentation files (the "Software"),
	to deal in the Software without restriction, including without limitation
	the rights to use, copy, modify, merge, publish, distribute, sublicense,
	and/or sell copies of the Software, and to permit persons to whom the
	Software is furnished to do so, subject to the following conditions:

	The above copyright notice and this permission notice shall be included
	in all copies or substantial portions of the Software.

	THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS
	OR IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY,
	FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT.  IN NO EVENT SHALL
	BRIAN PAUL BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER LIABILITY, WHETHER IN
	AN ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM, OUT OF OR IN
	CONNECTION WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN THE SOFTWARE.
*/

#include "VulkanBackend.h"
#include "ShaderCompilerCommon.h"

#include "VulkanCommon.h"

#include "CrossCompilerCommon.h"

const char* VULKAN_SUBPASS_FETCH[9] = 
{
	"VulkanSubpassDepthFetch",
	"VulkanSubpassFetch0",
	"VulkanSubpassFetch1",
	"VulkanSubpassFetch2",
	"VulkanSubpassFetch3",
	"VulkanSubpassFetch4",
	"VulkanSubpassFetch5",
	"VulkanSubpassFetch6",
	"VulkanSubpassFetch7"
};
const char* VULKAN_SUBPASS_FETCH_VAR[9] = 
{
	"GENERATED_SubpassDepthFetchAttachment",
	"GENERATED_SubpassFetchAttachment0",
	"GENERATED_SubpassFetchAttachment1",
	"GENERATED_SubpassFetchAttachment2",
	"GENERATED_SubpassFetchAttachment3",
	"GENERATED_SubpassFetchAttachment4",
	"GENERATED_SubpassFetchAttachment5",
	"GENERATED_SubpassFetchAttachment6",
	"GENERATED_SubpassFetchAttachment7"
};
const TCHAR* VULKAN_SUBPASS_FETCH_VAR_W[9] =
{
	TEXT("GENERATED_SubpassDepthFetchAttachment"),
	TEXT("GENERATED_SubpassFetchAttachment0"),
	TEXT("GENERATED_SubpassFetchAttachment1"),
	TEXT("GENERATED_SubpassFetchAttachment2"),
	TEXT("GENERATED_SubpassFetchAttachment3"),
	TEXT("GENERATED_SubpassFetchAttachment4"),
	TEXT("GENERATED_SubpassFetchAttachment5"),
	TEXT("GENERATED_SubpassFetchAttachment6"),
	TEXT("GENERATED_SubpassFetchAttachment7")
};
