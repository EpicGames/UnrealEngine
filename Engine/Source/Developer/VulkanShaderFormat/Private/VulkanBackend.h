// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VulkanCommon.h"

// Generates Vulkan compliant code from IR tokens
#ifdef __GNUC__
#pragma GCC visibility push(default)
#endif // __GNUC__

// InputAttachments
// 0 - reserved for depth input, 1-8 for color
extern const char* VULKAN_SUBPASS_FETCH[9];
extern const char* VULKAN_SUBPASS_FETCH_VAR[9];
extern const TCHAR* VULKAN_SUBPASS_FETCH_VAR_W[9];

#ifdef __GNUC__
#pragma GCC visibility pop
#endif // __GNUC__
