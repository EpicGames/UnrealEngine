// Copyright Epic Games, Inc. All Rights Reserved.
// .

#include "RHIShaderFormatDefinitions.inl"
#include "ShaderCompilerCommon.h"
#include "ShaderCompilerDefinitions.h"
#include "ShaderParameterParser.h"
#include "ShaderPreprocessTypes.h"
#include "SpirvReflectCommon.h"
#include "VulkanCommon.h"

#include "VulkanThirdParty.h"
#include "VulkanBackend.h"
#include "VulkanShaderResources.h"
#include "ShaderSDCE.h"

#include "SpirVShaderCompiler.inl"


inline bool IsVulkanShaderFormat(FName ShaderFormat)
{
	return ShaderFormat == NAME_VULKAN_ES3_1_ANDROID
		|| ShaderFormat == NAME_VULKAN_ES3_1
		|| ShaderFormat == NAME_VULKAN_SM5
		|| ShaderFormat == NAME_VULKAN_SM6
		|| ShaderFormat == NAME_VULKAN_SM5_ANDROID;
}

inline bool IsAndroidShaderFormat(FName ShaderFormat)
{
	return ShaderFormat == NAME_VULKAN_ES3_1_ANDROID
		|| ShaderFormat == NAME_VULKAN_SM5_ANDROID;
}


enum class EVulkanShaderVersion
{
	ES3_1,
	ES3_1_ANDROID,
	SM5,
	SM5_ANDROID,
	SM6,
	Invalid,
};


DEFINE_LOG_CATEGORY_STATIC(LogVulkanShaderCompiler, Log, All); 


class FVulkanShaderCompilerInternalState : public FSpirvShaderCompilerInternalState
{
	EVulkanShaderVersion FormatToVersion(FName Format)
	{
		if (Format == NAME_VULKAN_ES3_1)
		{
			return EVulkanShaderVersion::ES3_1;
		}
		else if (Format == NAME_VULKAN_ES3_1_ANDROID)
		{
			return EVulkanShaderVersion::ES3_1_ANDROID;
		}
		else if (Format == NAME_VULKAN_SM5_ANDROID)
		{
			return EVulkanShaderVersion::SM5_ANDROID;
		}
		else if (Format == NAME_VULKAN_SM5)
		{
			return EVulkanShaderVersion::SM5;
		}
		else if (Format == NAME_VULKAN_SM6)
		{
			return EVulkanShaderVersion::SM6;
		}
		else
		{
			FString FormatStr = Format.ToString();
			checkf(0, TEXT("Invalid shader format passed to Vulkan shader compiler: %s"), *FormatStr);
			return EVulkanShaderVersion::Invalid;
		}
	}

public:
	FVulkanShaderCompilerInternalState(const FShaderCompilerInput& InInput, const FShaderParameterParser* InParameterParser)
		: FSpirvShaderCompilerInternalState(InInput, InParameterParser)
	{
		Version = FormatToVersion(Input.ShaderFormat);

		if (Version == EVulkanShaderVersion::SM6)
		{
			MinimumTargetEnvironment = CrossCompiler::FShaderConductorOptions::ETargetEnvironment::Vulkan_1_3;
		}
		else if (Input.IsRayTracingShader() || Input.Environment.CompilerFlags.Contains(CFLAG_InlineRayTracing))
		{
			MinimumTargetEnvironment = CrossCompiler::FShaderConductorOptions::ETargetEnvironment::Vulkan_1_2;
		}
		else
		{
			MinimumTargetEnvironment = CrossCompiler::FShaderConductorOptions::ETargetEnvironment::Vulkan_1_1;
		}

		bIsAndroid = IsAndroidShaderFormat(Input.ShaderFormat);

		bSupportsOfflineCompiler = 
			Input.ShaderFormat == NAME_VULKAN_ES3_1_ANDROID ||
			Input.ShaderFormat == NAME_VULKAN_ES3_1 ||
			Input.ShaderFormat == NAME_VULKAN_SM5_ANDROID;;
	}

	bool IsSM6() const override final
	{
		return (Version == EVulkanShaderVersion::SM6);
	}

	bool IsSM5() const override final
	{
		return (Version == EVulkanShaderVersion::SM5) || (Version == EVulkanShaderVersion::SM5_ANDROID);
	}

	bool IsMobileES31() const override final
	{
		return (Version == EVulkanShaderVersion::ES3_1 || Version == EVulkanShaderVersion::ES3_1_ANDROID);
	}

	CrossCompiler::FShaderConductorOptions::ETargetEnvironment GetMinimumTargetEnvironment() const override final
	{
		return MinimumTargetEnvironment;
	}

	bool IsAndroid() const override final
	{
		return bIsAndroid;
	}

	bool SupportsOfflineCompiler() const override final
	{
		return bSupportsOfflineCompiler;
	}

private:
	EVulkanShaderVersion Version;
	CrossCompiler::FShaderConductorOptions::ETargetEnvironment MinimumTargetEnvironment;
	bool bIsAndroid = false;
	bool bSupportsOfflineCompiler = false;;
};


void ModifyVulkanCompilerInput(FShaderCompilerInput& Input)
{
	FVulkanShaderCompilerInternalState InternalState(Input, nullptr);
	SpirvShaderCompiler::ModifyCompilerInput(InternalState, Input);
}

// Helper function to know how much space to set aside in the shader record for a global
static uint32 GetSizeForType(FStringView TypeName, FStringView ArraySize)
{
	static TMap<FStringView, uint32> SizeForTypeMap;
	if (SizeForTypeMap.Num() == 0)
	{
		SizeForTypeMap.Add(FStringView(TEXT("uint")),   4);
		SizeForTypeMap.Add(FStringView(TEXT("uint2")),  8);
		SizeForTypeMap.Add(FStringView(TEXT("uint3")),  12);
		SizeForTypeMap.Add(FStringView(TEXT("uint4")),  16);
		SizeForTypeMap.Add(FStringView(TEXT("float")),  4);
		SizeForTypeMap.Add(FStringView(TEXT("float2")), 8);
		SizeForTypeMap.Add(FStringView(TEXT("float3")), 12);
		SizeForTypeMap.Add(FStringView(TEXT("float4")), 16);
	}

	checkf(ArraySize.Len() == 0, TEXT("Need to add array support!")); // :todo-jn: Add array parsing

	const uint32* TypeSize = SizeForTypeMap.Find(TypeName);
	checkf(TypeSize, TEXT("Missing type size for %.*s"), TypeName.Len(), TypeName.GetData());
	return *TypeSize;
}


// Add the globals as member in the VulkanShaderRecordLooseParameterData structure in the shader record
static bool ConvertGlobalsToShaderRecord(const FShaderCompilerInput& Input, const FShaderParameterParser& ShaderParameterParser, const TMap<FStringView, FStringView>& ReplacedGlobals, FString& PreprocessedShaderSource, FShaderCompilerOutput& Output)
{
	FString ShaderRecordGlobalsString;
	for (const auto& ParamDecl : ReplacedGlobals)
	{
		ShaderRecordGlobalsString += ParamDecl.Value;
	}

	if (ShaderRecordGlobalsString.Len())
	{
		const int32 ReplacementCount = PreprocessedShaderSource.ReplaceInline(TEXT("uint VulkanShaderRecordDummyGlobals;"), *ShaderRecordGlobalsString, ESearchCase::CaseSensitive);
		checkf(ReplacementCount == 1, TEXT("VulkanShaderRecordDummyGlobals was replaced %d times!"), ReplacementCount);
		return true;
	}

	return false;
}


struct FVulkanShaderParameterParserPlatformConfiguration : public FShaderParameterParser::FPlatformConfiguration
{
	FVulkanShaderParameterParserPlatformConfiguration(const FShaderCompilerInput& Input, TMap<FStringView, FStringView>& InReplacedGlobals)
		: FShaderParameterParser::FPlatformConfiguration()
		, bIsRayTracingShader(Input.IsRayTracingShader())
		, HitGroupSystemIndexBufferName(FShaderParameterParser::kBindlessSRVPrefix + FString(TEXT("HitGroupSystemIndexBuffer")))
		, HitGroupSystemVertexBufferName(FShaderParameterParser::kBindlessSRVPrefix + FString(TEXT("HitGroupSystemVertexBuffer")))
		, ReplacedGlobals(InReplacedGlobals)
	{
		EnumAddFlags(Flags, EShaderParameterParserConfigurationFlags::SupportsBindless | EShaderParameterParserConfigurationFlags::BindlessUsesArrays);

		// Create a _RootShaderParameters and bind it in slot 0 like any other uniform buffer
		if (Input.Target.GetFrequency() == SF_RayGen && Input.RootParametersStructure != nullptr)
		{
			StableConstantBufferType = TEXTVIEW("cbuffer");
			EnumAddFlags(Flags, EShaderParameterParserConfigurationFlags::UseStableConstantBuffer);
		}

		// Place loose data params in the shader record for shaders with bindless UBs
		if (bIsRayTracingShader && (Input.Target.GetFrequency() != SF_RayGen))
		{
			EnumAddFlags(Flags, EShaderParameterParserConfigurationFlags::ReplaceGlobals);
		}
	}
	virtual FString GenerateBindlessAccess(EBindlessConversionType BindlessType, FStringView FullTypeString, FStringView ArrayNameOverride, FStringView IndexString) const final
	{
		// GetSRVFromHeap(Type, Index) VULKAN_SRV_HEAP(Type)[VULKAN_HEAP_ACCESS(Index)]
		// GetUAVFromHeap(Type, Index) VULKAN_UAV_HEAP(Type)[VULKAN_HEAP_ACCESS(Index)]
		// GetSamplerFromHeap(Type, Index) VULKAN_SAMPLER_HEAP(Type)[VULKAN_HEAP_ACCESS(Index)]

		if (bIsRayTracingShader)
		{
			if (BindlessType == EBindlessConversionType::SRV)
			{
				// Patch the HitGroupSystemIndexBuffer/HitGroupSystemVertexBuffer indices to use the ones contained in the shader record
				if (IndexString == HitGroupSystemIndexBufferName)
				{
					IndexString = TEXTVIEW("VulkanHitGroupSystemParameters.BindlessHitGroupSystemIndexBuffer");
				}
				else if (IndexString == HitGroupSystemVertexBufferName)
				{
					IndexString = TEXTVIEW("VulkanHitGroupSystemParameters.BindlessHitGroupSystemVertexBuffer");
				}
			}

			// Raytracing shaders need NonUniformResourceIndex because bindless index can be divergent in hit/miss/callable shaders
			return FString::Printf(TEXT("%.*s[NonUniformResourceIndex(%.*s)]"),
				ArrayNameOverride.Len(), ArrayNameOverride.GetData(),
				IndexString.Len(), IndexString.GetData());
		}

		return FString::Printf(TEXT("%.*s[%.*s]"),
			ArrayNameOverride.Len(), ArrayNameOverride.GetData(),
			IndexString.Len(), IndexString.GetData());
	}

	// Fill the global with the value stored in the shader record
	virtual FString ReplaceGlobal(FStringView FullDeclString, FStringView ParamName) const final
	{
		FString NewDecl(FullDeclString);
		if (!FullDeclString.Contains(TEXT("ConstantBuffer")))
		{
			ReplacedGlobals.Add(ParamName, FullDeclString);

			NewDecl = TEXT("static ") + NewDecl;
			NewDecl.InsertAt(NewDecl.Find(TEXT(";")), FString::Printf(TEXT(" = VulkanHitGroupSystemParameters.Globals.%.*s"), ParamName.Len(), ParamName.GetData()));
		}
		return NewDecl;
	}

	const bool bIsRayTracingShader;
	const FString HitGroupSystemIndexBufferName;
	const FString HitGroupSystemVertexBufferName;
	TMap<FStringView, FStringView>& ReplacedGlobals;
};

void CompileVulkanShader(const FShaderCompilerInput& Input, const FShaderPreprocessOutput& InPreprocessOutput, FShaderCompilerOutput& Output)
{
	check(IsVulkanShaderFormat(Input.ShaderFormat));

	FString EntryPointName = Input.EntryPointName;
	FString PreprocessedSource(InPreprocessOutput.GetSourceViewWide());

	// Perform symbolic elimination
	const bool bSDCEMinified = UE::ShaderMinifier::SDCE::MinifyInPlace(Input, InPreprocessOutput, PreprocessedSource);

	TMap<FStringView, FStringView> ReplacedGlobals; // Note: these FStringView point to memory in FShaderParameterParser
	FVulkanShaderParameterParserPlatformConfiguration PlatformConfiguration(Input, ReplacedGlobals);
	FShaderParameterParser ShaderParameterParser(PlatformConfiguration);
	if (!ShaderParameterParser.ParseAndModify(Input, Output.Errors, PreprocessedSource))
	{
		// The FShaderParameterParser will add any relevant errors.
		return;
	}

	FVulkanShaderCompilerInternalState InternalState(Input, &ShaderParameterParser);

	if (InternalState.bUseStaticUniformBufferBindings)
	{
		// Match uniform buffers listed in FRaytracingShaderBindingLayout
		TArray<FString> RayTracingStaticUniformBufferNames;
		RayTracingStaticUniformBufferNames.Add(TEXT("Scene"));
		RayTracingStaticUniformBufferNames.Add(TEXT("View"));
		RayTracingStaticUniformBufferNames.Add(TEXT("NaniteRayTracing"));
		RayTracingStaticUniformBufferNames.Add(TEXT("LumenHardwareRayTracingUniformBuffer"));

		InternalState.PushConstantUBs = SpirvShaderCompiler::ConvertUBToBindless(PreprocessedSource, SpirvShaderCompiler::EBindlessUniformBufferType::PushConstant, RayTracingStaticUniformBufferNames);
	}

	if (InternalState.bUseBindlessUniformBuffer)
	{
		InternalState.AllBindlessUBs = SpirvShaderCompiler::ConvertUBToBindless(PreprocessedSource, SpirvShaderCompiler::EBindlessUniformBufferType::ShaderRecord);
		InternalState.bHasShaderRecordGlobals = ConvertGlobalsToShaderRecord(Input, ShaderParameterParser, ReplacedGlobals, PreprocessedSource, Output);
	}

	if (ShaderParameterParser.DidModifyShader() || bSDCEMinified || InternalState.AllBindlessUBs.Num() || InternalState.bHasShaderRecordGlobals || InternalState.PushConstantUBs.Num())
	{
		Output.ModifiedShaderSource = PreprocessedSource;
	}

	bool bSuccess = false;

	// Convert to ANSI prior to calling into ShaderConductor. This copy would have been incurred
	// by SC itself anyways, but would (will?) also be unnecessary if (when) shader parameter parser
	// is modified to operate on ANSI strings.
	const FShaderSource::FStringType PreprocessedSourceToCompile(PreprocessedSource);

#if PLATFORM_MAC || PLATFORM_WINDOWS || PLATFORM_LINUX
	// HitGroup shaders might have multiple entrypoints that we combine into a single blob
	if (InternalState.HasMultipleEntryPoints())
	{
		bSuccess = SpirvShaderCompiler::CompileShaderGroup(InternalState, PreprocessedSourceToCompile, Output);
	}
	else
	{
		// Compile regular shader via ShaderConductor (DXC)
		SpirvShaderCompilerSerializedOutput SerializedOutput;
		bSuccess = SpirvShaderCompiler::CompileWithShaderConductor(InternalState, PreprocessedSourceToCompile, SerializedOutput, Output);

		if (InternalState.bUseBindlessUniformBuffer)
		{
			SpirvShaderCompiler::UpdateBindlessUBs(InternalState, SerializedOutput, Output);
		}

		// Write out the header and shader source code (except for the extra shaders in hit groups)
		checkf(!(bSuccess && SerializedOutput.Spirv.Data.Num() == 0), TEXT("shader compilation was reported as successful but SPIR-V module is empty"));
		FMemoryWriter Ar(Output.ShaderCode.GetWriteAccess(), true);
		Ar << SerializedOutput.Header;
		Ar << SerializedOutput.ShaderResourceTable;

		uint32 SpirvCodeSizeBytes = SerializedOutput.Spirv.GetByteSize();
		Ar << SpirvCodeSizeBytes;
		if (SerializedOutput.Spirv.Data.Num() > 0)
		{
			Ar.Serialize((uint8*)SerializedOutput.Spirv.Data.GetData(), SpirvCodeSizeBytes);
		}

		SpirvShaderCompiler::FillShaderResourceUsageFlags(InternalState, SerializedOutput);
		Output.ShaderCode.AddOptionalData(SerializedOutput.PackedResourceCounts);
	}
#endif // PLATFORM_MAC || PLATFORM_WINDOWS || PLATFORM_LINUX
	
	if (InternalState.bUseBindlessUniformBuffer)
	{
		// HACK: Because of heavy code alterations with bindless ray tracing shaders, line numbers will be all over the place.  Remove the tag that leads to remapping...
		for (FShaderCompilerError& ErrorMsg : Output.Errors)
		{
			ErrorMsg.StrippedErrorMessage.ReplaceInline(TEXT("__UE_FILENAME_SENTINEL"), *Input.GenerateShaderName());
		}
	}

	if (Input.Environment.CompilerFlags.Contains(CFLAG_ExtraShaderData))
	{
		Output.ShaderCode.AddOptionalData(FShaderCodeName::Key, TCHAR_TO_UTF8(*Input.GenerateShaderName()));
	}

	Output.SerializeShaderCodeValidation();

	ShaderParameterParser.ValidateShaderParameterTypes(Input, InternalState.IsMobileES31(), Output);
	
	if (EnumHasAnyFlags(Input.DebugInfoFlags, EShaderDebugInfoFlags::CompileFromDebugUSF))
	{
		for (const auto& Error : Output.Errors)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%s\n"), *Error.GetErrorStringWithLineMarker());
		}
		ensure(bSuccess);
	}
}
