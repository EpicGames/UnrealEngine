// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanCommon.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/IShaderFormatModule.h"
#include "ShaderCore.h"
#include "ShaderCompilerCommon.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterParser.h"
#include "ShaderPreprocessor.h"
#include "ShaderPreprocessTypes.h"
#include "DXCWrapper.h"
#include "ShaderConductorContext.h"
#include "RHIShaderFormatDefinitions.inl"

extern void ModifyVulkanCompilerInput(FShaderCompilerInput& Input);

extern void CompileVulkanShader(
	const FShaderCompilerInput& Input,
	const FShaderPreprocessOutput& InPreprocessOutput,
	FShaderCompilerOutput& Output);

static const FGuid UE_SHADER_VULKAN_ES3_1_VER = FGuid("B84F72C8-3ECD-411E-993C-D7C7CEE26F28");
static const FGuid UE_SHADER_VULKAN_SM5_VER = FGuid("0715D8EE-9907-4A25-93AD-A3902C8E069A");
static const FGuid UE_SHADER_VULKAN_SM6_VER = FGuid("506C4300-261F-4B9B-97AE-3921B3CFFF34");

class FShaderFormatVulkan : public UE::ShaderCompilerCommon::FBaseShaderFormat
{
	FGuid InternalGetVersion(FName Format) const
	{
		if (Format == NAME_VULKAN_SM6)
		{
			return UE_SHADER_VULKAN_SM6_VER;
		}

		if (Format == NAME_VULKAN_SM5 || Format == NAME_VULKAN_SM5_ANDROID)
		{
			return UE_SHADER_VULKAN_SM5_VER;
		}

		if (Format == NAME_VULKAN_ES3_1_ANDROID || Format == NAME_VULKAN_ES3_1)
		{
			return UE_SHADER_VULKAN_ES3_1_VER;
		}

		FString FormatStr = Format.ToString();
		checkf(0, TEXT("Invalid shader format passed to Vulkan shader compiler: %s"), *FormatStr);
		return FGuid();
	}

	uint32 ShaderConductorVersionHash;

public:

	FShaderFormatVulkan(uint32 InShaderConductorVersionHash)
		: ShaderConductorVersionHash(InShaderConductorVersionHash)
	{
	}

	virtual uint32 GetVersion(FName Format) const override
	{
		uint32 Version = GetTypeHash(InternalGetVersion(Format));
		Version = HashCombine(Version, GetTypeHash(ShaderConductorVersionHash));

	#if VULKAN_ENABLE_BINDING_DEBUG_NAMES
		Version = HashCombine(Version, 0xFC0848E2);
	#endif

		return Version;
	}
	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const
	{
		OutFormats.Add(NAME_VULKAN_SM5);
		OutFormats.Add(NAME_VULKAN_ES3_1_ANDROID);
		OutFormats.Add(NAME_VULKAN_ES3_1);
		OutFormats.Add(NAME_VULKAN_SM5_ANDROID);
		OutFormats.Add(NAME_VULKAN_SM6);
	}

	virtual void ModifyShaderCompilerInput(FShaderCompilerInput& Input) const override
	{
		ModifyVulkanCompilerInput(Input);
	}

	virtual void CompilePreprocessedShader(const FShaderCompilerInput& Input, const FShaderPreprocessOutput& PreprocessOutput, FShaderCompilerOutput& Output) const override
	{
		CompileVulkanShader(Input, PreprocessOutput, Output);
	}

	virtual const TCHAR* GetPlatformIncludeDirectory() const
	{
		return TEXT("Vulkan");
	}
};

/**
 * Module for Vulkan shaders
 */

static IShaderFormat* Singleton = nullptr;

class FVulkanShaderFormatModule : public IShaderFormatModule, public FShaderConductorModuleWrapper
{
public:
	virtual ~FVulkanShaderFormatModule()
	{
		delete Singleton;
		Singleton = nullptr;
	}

	virtual IShaderFormat* GetShaderFormat()
	{
		if (!Singleton)
		{
			Singleton = new FShaderFormatVulkan(FShaderConductorModuleWrapper::GetModuleVersionHash());
		}

		return Singleton;
	}

	virtual void ShutdownModule() override
	{
		CrossCompiler::FShaderConductorContext::Shutdown();
	}
};

IMPLEMENT_MODULE( FVulkanShaderFormatModule, VulkanShaderFormat);
