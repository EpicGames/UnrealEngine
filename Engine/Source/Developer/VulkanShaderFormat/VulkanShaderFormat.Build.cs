// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VulkanShaderFormat : ModuleRules
{
	public VulkanShaderFormat(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("TargetPlatform");

		// Do not link the module (as that would require the vulkan dll), only the include paths
		PublicIncludePathModuleNames.Add("VulkanRHI");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"RenderCore",
				"ShaderCompilerCommon",
				"ShaderPreprocessor",
				"RHI", // @todo platplug: This would not be needed if we could move FDataDriveShaderPlatformInfo (and ERHIFeatureLevel) into RenderCore or maybe its own module?
			}
			);

		if (Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Linux)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "SPIRVReflect");
		}

		if (Target.Platform != UnrealTargetPlatform.Win64 &&
			Target.Platform != UnrealTargetPlatform.Android &&
			!Target.IsInPlatformGroup(UnrealPlatformGroup.Linux) &&
			Target.Platform != UnrealTargetPlatform.Mac)
		{
			PrecompileForTargets = PrecompileTargetsType.None;
		}

		AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan");
	}
}
